// Copyright Epic Games, Inc. All Rights Reserved.

#include "AndroidMoviePlayer.h"
#include "AndroidMovieStreamer.h"

#include "Misc/CoreDelegates.h"

TSharedPtr<FAndroidMediaPlayerStreamer, ESPMode::ThreadSafe> AndroidMovieStreamer;

class FAndroidMoviePlayerModule : public IModuleInterface
{
	/** IModuleInterface implementation */
	virtual void StartupModule() override
	{
		if (IsSupported())
		{
            FAndroidMediaPlayerStreamer* Streamer = new FAndroidMediaPlayerStreamer;
			AndroidMovieStreamer = MakeShareable(Streamer);
			
            FCoreDelegates::RegisterMovieStreamerDelegate.Broadcast(AndroidMovieStreamer);
		}
	}

	virtual void ShutdownModule() override
	{
        if (AndroidMovieStreamer.IsValid())
        {
            FCoreDelegates::UnRegisterMovieStreamerDelegate.Broadcast(AndroidMovieStreamer);
        }

		if (IsSupported())
		{
              AndroidMovieStreamer->Cleanup();
		}

        AndroidMovieStreamer.Reset();
	}

private:

	bool IsSupported()
	{
		return FAndroidMisc::GetAndroidBuildVersion() >= 14;
	}
};

IMPLEMENT_MODULE( FAndroidMoviePlayerModule, AndroidMoviePlayer )
