// Copyright Epic Games, Inc. All Rights Reserved.

#include "AndroidMovieStreamer.h"

#include "Android/AndroidApplication.h"
#include "Android/AndroidJava.h"
#include "Android/AndroidPlatformFile.h"

#include "Rendering/RenderingCommon.h"
#include "RenderingThread.h"
#include "RenderUtils.h"
#include "Slate/SlateTextures.h"
#include "MoviePlayer.h"
#include "Containers/StringConv.h"

#include "Misc/ScopeLock.h"
#include "Misc/Paths.h"

#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "IPlatformFilePak.h"

DEFINE_LOG_CATEGORY_STATIC(LogAndroidMediaPlayerStreamer, Log, All);

#define MOVIE_FILE_EXTENSION "mp4"

FAndroidMediaPlayerStreamer::FAndroidMediaPlayerStreamer()
	: JavaMediaPlayer(nullptr)
	, CurrentPosition(-1)
{
	JavaMediaPlayer = MakeShareable(new FJavaAndroidMediaPlayer(false, FAndroidMisc::ShouldUseVulkan(), false, true));
	MovieViewport = MakeShareable(new FMovieViewport());
}

FAndroidMediaPlayerStreamer::~FAndroidMediaPlayerStreamer()
{
}

bool FAndroidMediaPlayerStreamer::Init(const TArray<FString>& MoviePaths, TEnumAsByte<EMoviePlaybackType> inPlaybackType)
{
	{
		FScopeLock Lock(&MovieQueueCriticalSection);
		MovieQueue.Append(MoviePaths);
	}
	return StartNextMovie();
}

void FAndroidMediaPlayerStreamer::ForceCompletion()
{
	CloseMovie();
}

static void DoUpdateTextureMovieSampleExecute(TWeakPtr<FJavaAndroidMediaPlayer, ESPMode::ThreadSafe> JavaMediaPlayerPtr, int32 DestTexture)
{
	auto PinnedJavaMediaPlayer = JavaMediaPlayerPtr.Pin();

	if (!PinnedJavaMediaPlayer.IsValid())
	{
		return;
	}

	PinnedJavaMediaPlayer->GetVideoLastFrame(DestTexture);
}

struct FRHICommandUpdateTextureMovieSample final : public FRHICommand<FRHICommandUpdateTextureMovieSample>
{
	TWeakPtr<FJavaAndroidMediaPlayer, ESPMode::ThreadSafe> JavaMediaPlayerPtr;
	FSlateTexture2DRHIRef* DestTexture;

	FORCEINLINE_DEBUGGABLE FRHICommandUpdateTextureMovieSample(TWeakPtr<FJavaAndroidMediaPlayer, ESPMode::ThreadSafe> InJavaMediaPlayerPtr, FSlateTexture2DRHIRef* InDestTexture)
		: JavaMediaPlayerPtr(InJavaMediaPlayerPtr)
		, DestTexture(InDestTexture)
	{
	}
	void Execute(FRHICommandListBase& CmdList)
	{
		int32 NativeDestTexture = *reinterpret_cast<int32*>(DestTexture->GetTypedResource().GetReference()->GetNativeResource());

		QUICK_SCOPE_CYCLE_COUNTER(STAT_FRHICommandUpdateTextureMovieSample_Execute);
		DoUpdateTextureMovieSampleExecute(JavaMediaPlayerPtr, NativeDestTexture);
	}
};

bool FAndroidMediaPlayerStreamer::Tick(FRHICommandListBase& RHICmdList, float DeltaTime)
{
	// Check the list of textures pending deletion and remove any that are no longer valid
	for (int32 TextureIndex = 0; TextureIndex < TexturesPendingDeletion.Num();)
	{
		TSharedPtr<FSlateTexture2DRHIRef, ESPMode::ThreadSafe> Tex = TexturesPendingDeletion[TextureIndex];
		if (!Tex->IsInitialized())
		{
			// Texture can now be removed from the list
			TexturesPendingDeletion.RemoveAt(TextureIndex);

			// Don't increment i, since element 'i' wasn't just filled by the next element
		}
		else
		{
			// Advance to the next one
			++TextureIndex;
		}
	}

	FSlateTexture2DRHIRef* CurrentTexture = Texture.Get();

	if (IsInRenderingThread())
	{
		int32 NextPosition = JavaMediaPlayer->GetCurrentPosition();
		if (CurrentPosition != NextPosition)
		{
			// Movie is on a different frame than what we have. We
			// need to read the new frame data and post it to Slate.

			if (!CurrentTexture->IsInitialized())
			{
				CurrentTexture->InitResource(RHICmdList);
			}

			if (!FAndroidMisc::ShouldUseVulkan())
			{
				if (IsRunningRHIInSeparateThread())
				{
					new (RHICmdList.AllocCommand<FRHICommandUpdateTextureMovieSample>()) FRHICommandUpdateTextureMovieSample(JavaMediaPlayer, CurrentTexture);
				}
				else
				{
					int32 DestTexture = *reinterpret_cast<int32*>(CurrentTexture->GetTypedResource().GetReference()->GetNativeResource());
					DoUpdateTextureMovieSampleExecute(JavaMediaPlayer, DestTexture);
				}
			}
			else
			{
				uint32 Stride = 0;
				int64 SampleCount = 0;
				void* LastFrameData = nullptr;
				if(JavaMediaPlayer->GetVideoLastFrameData(LastFrameData, SampleCount))
				{
					void* DestTextureData = RHICmdList.GetAsImmediate().LockTexture2D(CurrentTexture->GetTypedResource(), 0, RLM_WriteOnly, Stride, false);
					FMemory::Memcpy(DestTextureData, LastFrameData, SampleCount);
					RHICmdList.GetAsImmediate().UnlockTexture2D(CurrentTexture->GetTypedResource(), 0, false);
				}
			}

			CurrentPosition = NextPosition;
		}
	}

	if (!JavaMediaPlayer->IsPlaying())
	{
		// Current movie finished playing. Clean up current movie and move on to
		// the next movie, if any.
		CloseMovie();
		if (!StartNextMovie())
		{
			// Failed to start the next movie, need to indicate that
			// we are done.
			return true;
		}
	}

	return false; //Not finished
}

TSharedPtr<class ISlateViewport> FAndroidMediaPlayerStreamer::GetViewportInterface()
{
	return MovieViewport;
}

float FAndroidMediaPlayerStreamer::GetAspectRatio() const
{
	return (float)MovieViewport->GetSize().X / (float)MovieViewport->GetSize().Y;
}

void FAndroidMediaPlayerStreamer::Cleanup()
{
}

bool FAndroidMediaPlayerStreamer::StartNextMovie()
{
	FString MoviePath;
	{
		FScopeLock QueueLock(&MovieQueueCriticalSection);
		if (MovieQueue.Num() == 0)
		{
			return false;
		}

		// Construct a canonical path for the movie.
		MoviePath
			= FPaths::ProjectContentDir() + FString("Movies/")
			+ MovieQueue[0] + FString(".") + FString(MOVIE_FILE_EXTENSION);
		FPaths::NormalizeFilename(MoviePath);
		MovieQueue.RemoveAt(0);
	}

	bool movieOK = true;

	IAndroidPlatformFile& PlatformFile = IAndroidPlatformFile::GetPlatformPhysical();

	// Don't bother trying to play it if we can't find it.
	if (!PlatformFile.FileExists(*MoviePath))
	{
		// possible it is in a PAK file
		FPakPlatformFile* PakPlatformFile = (FPakPlatformFile*)(FPlatformFileManager::Get().FindPlatformFile(FPakPlatformFile::GetTypeName()));

		TRefCountPtr<FPakFile> PakFile;
		FPakEntry FileEntry;
		if (PakPlatformFile == nullptr || !PakPlatformFile->FindFileInPakFiles(*MoviePath, &PakFile, &FileEntry))
		{
			return false;
		}

		// is it a simple case (can just use file datasource)?
		if (FileEntry.CompressionMethodIndex == 0 && !FileEntry.IsEncrypted())
		{
			FString PakFilename = PakFile->GetFilename();
			int64 PakHeaderSize = FileEntry.GetSerializedSize(PakFile->GetInfo().Version);
			int64 OffsetInPak = FileEntry.Offset + PakHeaderSize;
			int64 FileSize = FileEntry.Size;

			int64 FileOffset = PlatformFile.FileStartOffset(*PakFilename) + OffsetInPak;
			FString FileRootPath = PlatformFile.FileRootPath(*PakFilename);

			if (!JavaMediaPlayer->SetDataSource(FileRootPath, FileOffset, FileSize))
			{
				return false;
			}
		}
		else
		{
			// only support media data source on Android 6.0+
			if (FAndroidMisc::GetAndroidBuildVersion() < 23)
			{
				//UE_LOGF(LogAndroidMedia, Warning, "File cannot be played on this version of Android due to PAK file with compression or encryption: %ls.", *MoviePath);
				return false;
			}

			TSharedRef<FArchive, ESPMode::ThreadSafe> Archive = MakeShareable(IFileManager::Get().CreateFileReader(*MoviePath));

			if (!JavaMediaPlayer->SetDataSource(Archive))
			{
				//UE_LOGF(LogAndroidMedia, Warning, "Failed to set data source for archive %ls", *MoviePath);
				return false;
			}
		}
	}
	else
	{
		// Get information about the movie.
		int64 FileOffset = PlatformFile.FileStartOffset(*MoviePath);
		int64 FileSize = PlatformFile.FileSize(*MoviePath);
		FString FileRootPath = PlatformFile.FileRootPath(*MoviePath);

		// Play the movie as a file or asset.
		if (PlatformFile.IsAsset(*MoviePath))
		{
			movieOK = JavaMediaPlayer->SetDataSource(
				IAndroidPlatformFile::GetPlatformPhysical().GetAssetManager(),
				FileRootPath, FileOffset, FileSize);
		}
		else
		{
			movieOK = JavaMediaPlayer->SetDataSource(FileRootPath, FileOffset, FileSize);
		}
	}

	FIntPoint VideoDimensions;
	if (movieOK)
	{
		JavaMediaPlayer->Prepare();
		VideoDimensions.X = JavaMediaPlayer->GetVideoWidth();
		VideoDimensions.Y = JavaMediaPlayer->GetVideoHeight();
		movieOK = VideoDimensions != FIntPoint::ZeroValue;
	}
	if (movieOK)
	{
		Texture = MakeShareable(new FSlateTexture2DRHIRef(
			VideoDimensions.X, VideoDimensions.Y,
			PF_B8G8R8A8,
			NULL,
			TexCreate_RenderTargetable | TexCreate_ShaderResource,
			true));

		uint32 FrameBytes = VideoDimensions.X * VideoDimensions.Y * GPixelFormats[PF_B8G8R8A8].BlockBytes;

		FSlateTexture2DRHIRef* TextureRHIRef = Texture.Get();
		ENQUEUE_RENDER_COMMAND(InitMovieTexture)(
			[TextureRHIRef, FrameBytes](FRHICommandListImmediate& RHICmdList)
			{
				TextureRHIRef->InitResource(RHICmdList);
	
				// clear texture to black
				uint32 Stride = 0;
				void* TextureBuffer = RHICmdList.LockTexture2D(TextureRHIRef->GetTypedResource(), 0, RLM_WriteOnly, Stride, false);
				FMemory::Memset(TextureBuffer, 0, FrameBytes);
				RHICmdList.UnlockTexture2D(TextureRHIRef->GetTypedResource(), 0, false);

				RHICmdList.Transition(FRHITransitionInfo(TextureRHIRef->GetTypedResource(), ERHIAccess::Unknown, ERHIAccess::SRVMask));
			});

		MovieViewport->SetTexture(Texture);
	}
	if (movieOK)
	{
		JavaMediaPlayer->Start();
	}
	if (!movieOK)
	{
		JavaMediaPlayer->Reset();
	}
	return movieOK;
}

void FAndroidMediaPlayerStreamer::CloseMovie()
{
	JavaMediaPlayer->Stop();
	JavaMediaPlayer->Reset();

	if (Texture.IsValid())
	{
		TexturesPendingDeletion.Add(Texture);

		MovieViewport->SetTexture(NULL);
		Texture.Reset();
	}
}

FString FAndroidMediaPlayerStreamer::GetMovieName()
{
	return MovieQueue.Num() > 0 ? MovieQueue[0] : TEXT("");
}

bool FAndroidMediaPlayerStreamer::IsLastMovieInPlaylist()
{
	return MovieQueue.Num() <= 1;
}

