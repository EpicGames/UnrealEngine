// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MoviePlayer.h"
#include "Android/AndroidJavaMediaPlayer.h"
#include "Slate/SlateTextures.h"

// The actual streamer class
class FAndroidMediaPlayerStreamer : public IMovieStreamer
{
public:
	FAndroidMediaPlayerStreamer();
	virtual ~FAndroidMediaPlayerStreamer();

	virtual bool Init(const TArray<FString>& MoviePaths, TEnumAsByte<EMoviePlaybackType> inPlaybackType) override;
	virtual void ForceCompletion() override;
	virtual bool Tick(FRHICommandListBase& RHICmdList, float DeltaTime) override;
	virtual TSharedPtr<class ISlateViewport> GetViewportInterface() override;
	virtual float GetAspectRatio() const override;
	virtual void Cleanup() override;

	virtual FString GetMovieName() override;
	virtual bool IsLastMovieInPlaylist() override;

	FOnCurrentMovieClipFinished OnCurrentMovieClipFinishedDelegate;
	virtual FOnCurrentMovieClipFinished& OnCurrentMovieClipFinished() override { return OnCurrentMovieClipFinishedDelegate; }

private:
	/**
	A list of all the stored movie paths we have enqueued for playing
	*/
	FCriticalSection			MovieQueueCriticalSection;
	TArray<FString>				MovieQueue;

	/**
	Texture and view-port data for displaying to Slate
	*/
	TSharedPtr<FMovieViewport>		MovieViewport;
	TSharedPtr<FSlateTexture2DRHIRef, ESPMode::ThreadSafe> Texture;

	/**
	List of textures pending deletion, need to keep this list because we can't immediately
	destroy them since they could be getting used on the main thread
	*/
	TArray<TSharedPtr<FSlateTexture2DRHIRef, ESPMode::ThreadSafe>> TexturesPendingDeletion;

	/**
	Java side interface for MediaPlayer utility.
	*/
	TSharedPtr<FJavaAndroidMediaPlayer, ESPMode::ThreadSafe> JavaMediaPlayer;

	/**
	Some state for the playing movie.
	*/
	int32 CurrentPosition;

	bool StartNextMovie();
	void CloseMovie();
};
